{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "Global Configuration Data",
	"Description" : "A system that is used to query configuration data that can come from many different sources without knowing specifically which one.",
	"Category": "Misc",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "http://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"ExplicitlyLoaded": false,
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"IsExperimentalVersion": true,
	"Installed" : false,
	"Modules": [
		{
			"Name": "GlobalConfigurationData",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "GlobalConfigurationDataCore",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
	]
}
