// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GlobalConfigurationData : ModuleRules
	{
        public GlobalConfigurationData(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"GlobalConfigurationDataCore"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Engine",
				});
		}
	}
}
