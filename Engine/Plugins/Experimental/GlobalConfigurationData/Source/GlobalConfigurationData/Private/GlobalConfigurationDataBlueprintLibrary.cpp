// Copyright Epic Games, Inc. All Rights Reserved.

#include "GlobalConfigurationDataBlueprintLibrary.h"
#include "GlobalConfigurationData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GlobalConfigurationDataBlueprintLibrary)

bool UGlobalConfigurationDataBlueprintLibrary::GetConfigDataBool(const FString& EntryName, bool &bValueOut)
{
	return UE::GlobalConfigurationData::TryGetData(EntryName, bValueOut);
}

bool UGlobalConfigurationDataBlueprintLibrary::GetConfigDataInt(const FString& EntryName, int32& ValueOut)
{
	return UE::GlobalConfigurationData::TryGetData(EntryName, ValueOut);
}

bool UGlobalConfigurationDataBlueprintLibrary::GetConfigDataFloat(const FString& EntryName, float& ValueOut)
{
	return UE::GlobalConfigurationData::TryGetData(EntryName, ValueOut);
}

bool UGlobalConfigurationDataBlueprintLibrary::GetConfigDataString(const FString& EntryName, FString& ValueOut)
{
	return UE::GlobalConfigurationData::TryGetData(EntryName, ValueOut);
}

bool UGlobalConfigurationDataBlueprintLibrary::GetConfigDataText(const FString& EntryName, FText& ValueOut)
{
	return UE::GlobalConfigurationData::TryGetData(EntryName, ValueOut);
}

bool UGlobalConfigurationDataBlueprintLibrary::GetConfigDataStruct(const FString& EntryName, UScriptStruct* StructType, FInstancedStruct& ValueOut)
{
	if (StructType)
	{
		ValueOut.InitializeAs(StructType);
		return UE::GlobalConfigurationData::TryGetDataOfType(EntryName, ValueOut.GetScriptStruct(), ValueOut.GetMutableMemory());
	}
	return false;
}

bool UGlobalConfigurationDataBlueprintLibrary::GetConfigDataObject(const FString& EntryName, UObject* ValueInOut)
{
	if (ValueInOut)
	{
		return UE::GlobalConfigurationData::TryGetDataOfType(EntryName, ValueInOut->GetClass(), ValueInOut);
	}
	return false;
}

bool UGlobalConfigurationDataBlueprintLibrary::GetConfigDataBoolWithDefault(const FString& EntryName, bool bDefaultValue)
{
	return UE::GlobalConfigurationData::GetDataWithDefault(EntryName, bDefaultValue);
}

int32 UGlobalConfigurationDataBlueprintLibrary::GetConfigDataIntWithDefault(const FString& EntryName, int32 DefaultValue)
{
	return UE::GlobalConfigurationData::GetDataWithDefault(EntryName, DefaultValue);
}

float UGlobalConfigurationDataBlueprintLibrary::GetConfigDataFloatWithDefault(const FString& EntryName, float DefaultValue)
{
	return UE::GlobalConfigurationData::GetDataWithDefault(EntryName, DefaultValue);
}

FString UGlobalConfigurationDataBlueprintLibrary::GetConfigDataStringWithDefault(const FString& EntryName, FString DefaultValue)
{
	return UE::GlobalConfigurationData::GetDataWithDefault(EntryName, DefaultValue);
}

FText UGlobalConfigurationDataBlueprintLibrary::GetConfigDataTextWithDefault(const FString& EntryName, FText DefaultValue)
{
	return UE::GlobalConfigurationData::GetDataWithDefault(EntryName, DefaultValue);
}
