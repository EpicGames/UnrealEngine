// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "GlobalConfigurationDataBlueprintLibrary.generated.h"

UCLASS()
class UGlobalConfigurationDataBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintPure, Category="Global Configuration Data")
	static bool GetConfigDataBool(const FString& EntryName, bool &bValueOut);

	UFUNCTION(BlueprintPure, Category="Global Configuration Data")
	static bool GetConfigDataInt(const FString& EntryName, int32& ValueOut);

	UFUNCTION(BlueprintPure, Category="Global Configuration Data")
	static bool GetConfigDataFloat(const FString& EntryName, float& ValueOut);

	UFUNCTION(BlueprintPure, Category="Global Configuration Data")
	static bool GetConfigDataString(const FString& EntryName, FString& ValueOut);

	UFUNCTION(BlueprintPure, Category="Global Configuration Data")
	static bool GetConfigDataText(const FString& EntryName, FText& ValueOut);

	UFUNCTION(BlueprintPure, Category="Global Configuration Data")
	static bool GetConfigDataStruct(const FString& EntryName, UScriptStruct* StructType, FInstancedStruct& ValueOut);

	UFUNCTION(BlueprintPure, Category="Global Configuration Data")
	static bool GetConfigDataObject(const FString& EntryName, UObject* ValueInOut);

	UFUNCTION(BlueprintPure, Category="Global Configuration Data")
	static bool GetConfigDataBoolWithDefault(const FString& EntryName, bool bDefaultValue);

	UFUNCTION(BlueprintPure, Category="Global Configuration Data")
	static int32 GetConfigDataIntWithDefault(const FString& EntryName, int32 DefaultValue);

	UFUNCTION(BlueprintPure, Category="Global Configuration Data")
	static float GetConfigDataFloatWithDefault(const FString& EntryName, float DefaultValue);

	UFUNCTION(BlueprintPure, Category="Global Configuration Data")
	static FString GetConfigDataStringWithDefault(const FString& EntryName, FString DefaultValue);

	UFUNCTION(BlueprintPure, Category="Global Configuration Data")
	static FText GetConfigDataTextWithDefault(const FString& EntryName, FText DefaultValue);
};
