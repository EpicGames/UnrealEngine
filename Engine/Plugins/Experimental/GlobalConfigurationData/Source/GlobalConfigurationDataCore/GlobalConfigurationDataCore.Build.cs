// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GlobalConfigurationDataCore : ModuleRules
	{
        public GlobalConfigurationDataCore(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Json",
					"JsonUtilities",
				});
		}
	}
}
