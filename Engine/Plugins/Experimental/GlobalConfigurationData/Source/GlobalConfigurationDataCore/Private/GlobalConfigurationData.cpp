// Copyright Epic Games, Inc. All Rights Reserved.

#include "GlobalConfigurationData.h"

#include "GlobalConfigurationDataInternal.h"
#include "GlobalConfigurationRouter.h"
#include "JsonObjectConverter.h"

bool UE::GlobalConfigurationData::TryGetDataOfType(const FString& EntryName, const UStruct* Type, void* DataOut)
{
	if (TSharedPtr<FJsonValue> Data = IGlobalConfigurationRouter::TryGetData(EntryName))
	{
		TSharedPtr<FJsonObject>* JsonObject;
		if (Data->TryGetObject(JsonObject))
		{
			return FJsonObjectConverter::JsonObjectToUStruct(JsonObject->ToSharedRef(), Type, DataOut);
		}
		else
		{
			UE_LOGFMT(LogGlobalConfigurationData, Error, "Failed to json deserialize key {EntryName} from value {EntryValue}", *EntryName, IGlobalConfigurationRouter::TryPrintString(Data));
		}
	}
	return false;
}

bool UE::GlobalConfigurationData::TryGetData(const FString& EntryName, bool& bValueOut)
{
	if (TSharedPtr<FJsonValue> Data = IGlobalConfigurationRouter::TryGetData(EntryName))
	{
		return Data->TryGetBool(bValueOut);
	}
	return false;
}

bool UE::GlobalConfigurationData::TryGetData(const FString& EntryName, int32& ValueOut)
{
	if (TSharedPtr<FJsonValue> Data = IGlobalConfigurationRouter::TryGetData(EntryName))
	{
		return Data->TryGetNumber(ValueOut);
	}
	return false;
}

bool UE::GlobalConfigurationData::TryGetData(const FString& EntryName, float& ValueOut)
{
	if (TSharedPtr<FJsonValue> Data = IGlobalConfigurationRouter::TryGetData(EntryName))
	{
		return Data->TryGetNumber(ValueOut);
	}
	return false;
}

bool UE::GlobalConfigurationData::TryGetData(const FString& EntryName, FString& ValueOut)
{
	if (TSharedPtr<FJsonValue> Data = IGlobalConfigurationRouter::TryGetData(EntryName))
	{
		return Data->TryGetString(ValueOut);
	}
	return false;
}

bool UE::GlobalConfigurationData::TryGetData(const FString& EntryName, FText& ValueOut)
{
	FString StringData;
	if (TryGetData(EntryName, StringData))
	{
		// Todo: Better interpretation for loc text
		ValueOut = FText::FromString(MoveTemp(StringData));
		return true;
	}
	return false;
}