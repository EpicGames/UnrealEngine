// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#include "GlobalConfigurationDataInternal.h"
#include "Logging/LogMacros.h"
#include "Routers/GlobalConfigurationConfigRouter.h"
#include "Routers/GlobalConfigurationConsoleCommandRouter.h"

DEFINE_LOG_CATEGORY(LogGlobalConfigurationData);

class FGlobalConfigurationDataCoreModule : public IModuleInterface
{
public:
	FGlobalConfigurationDataCoreModule()
	: LowPriorityConfigRouter(TEXT("GlobalConfigurationData"), TEXT("ConfigLow"), INT32_MIN)
	, HighPriorityConfigRouter(TEXT("GlobalConfigurationDataHotfix"), TEXT("ConfigHigh"), INT32_MAX-1) // Console command still wins since it's for debugging
	{
	}
	
private:
	// Just existing is enough for these
	FGlobalConfigurationConfigRouter LowPriorityConfigRouter;
	FGlobalConfigurationConfigRouter HighPriorityConfigRouter;

#if !UE_BUILD_SHIPPING
	FGlobalConfigurationConsoleCommandRouter ConsoleCommandRouter;
#endif
};

IMPLEMENT_MODULE(FGlobalConfigurationDataCoreModule, GlobalConfigurationDataCore);
