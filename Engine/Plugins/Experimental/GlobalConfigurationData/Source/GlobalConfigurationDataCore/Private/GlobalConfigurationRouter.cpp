// Copyright Epic Games, Inc. All Rights Reserved.

#include "GlobalConfigurationRouter.h"
#include "GlobalConfigurationDataInternal.h"

#include "HAL/IConsoleManager.h"
#include "JsonObjectConverter.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonSerializer.h"

namespace IGlobalConfigurationRouter_Private
{
	TArray<const IGlobalConfigurationRouter*> RegisteredRouters;

	class FStackJsonReader : public TJsonReader<>
	{
	public:
		FStackJsonReader(const FStringView& String)
		: TJsonReader(&Reader)
		, Reader((void*)String.GetData(), String.NumBytes(), false)
		{
		}
		FBufferReader Reader;
	};

	class FStackJsonWriter : public TJsonStringWriter<TCondensedJsonPrintPolicy<TCHAR>>
	{
	public:
		explicit FStackJsonWriter(FString* Out)
		: TJsonStringWriter(Out, 0)
		{
		}
	};

	static bool bShowParseErrorsAsWarning = false;
	static FAutoConsoleVariableRef CVarMaxSearchResultsToReport(
		TEXT("GCD.ShowParseErrorsAsWarning"),
		bShowParseErrorsAsWarning,
		TEXT("If enabled, json parse errors will be logged as warnings, otherwise they are logged as verbose."));

	static bool bAllowFlattenJsonObject = true;
	static FAutoConsoleVariableRef CVarAllowFlattenJsonObject(
		TEXT("GCD.AllowFlattenJsonObject"),
		bAllowFlattenJsonObject,
		TEXT("If enabled, assume that a json object with a single entry was auto flattened and accept the resulting value anyway."));
}

IGlobalConfigurationRouter::IGlobalConfigurationRouter(FString&& InRouterName, int32 InPriority)
: RouterName(MoveTemp(InRouterName))
, Priority(InPriority)
{
	check(IsInGameThread());
	IGlobalConfigurationRouter_Private::RegisteredRouters.Add(this);
	IGlobalConfigurationRouter_Private::RegisteredRouters.Sort([](const IGlobalConfigurationRouter& LHS, const IGlobalConfigurationRouter& RHS)
		{
			// Can't have routers with the same priority
			check(LHS.Priority != RHS.Priority);
			return LHS.Priority > RHS.Priority;
		});

	UE_LOGFMT(LogGlobalConfigurationData, Verbose, "Registered Global Configuration Router {RouterName}", RouterName);
}
IGlobalConfigurationRouter::~IGlobalConfigurationRouter()
{
	check(IsInGameThread());
	IGlobalConfigurationRouter_Private::RegisteredRouters.Remove(this);
	UE_LOGFMT(LogGlobalConfigurationData, Verbose, "Unregistered Global Configuration Router {RouterName}", RouterName);
}

TSharedPtr<FJsonValue> IGlobalConfigurationRouter::TryGetData(const FString& FullEntryName)
{
	check(IsInGameThread());

	FString ObjectName;
	FString FieldName;

	const FString& EntryName = FullEntryName.Split(TEXT("."), &ObjectName, &FieldName) ? ObjectName : FullEntryName;
	UE_CLOGFMT(!FieldName.IsEmpty(), LogGlobalConfigurationData, VeryVerbose, "IGlobalConfigurationRouter::TryGetData parsing full entry name into Entry: {EntryName}, Field: {FieldName}", EntryName, FieldName);

	for (const IGlobalConfigurationRouter* Router : IGlobalConfigurationRouter_Private::RegisteredRouters)
	{
		TSharedPtr<FJsonValue> Result = Router->TryGetDataFromRouter(EntryName);
		if (Result.IsValid())
		{
			if (!FieldName.IsEmpty())
			{
				if (Result->Type == EJson::Object)
				{
					if (TSharedPtr<FJsonValue> Field = Result->AsObject()->TryGetField(FieldName))
					{
						UE_LOGFMT(LogGlobalConfigurationData, VeryVerbose, "IGlobalConfigurationRouter::TryGetData found value for {EntryName} in router {RouterName}: {EntryValue}", FullEntryName, Router->RouterName, TryPrintString(Field));
						return Field;
					}
					else
					{
						UE_LOGFMT(LogGlobalConfigurationData, Warning, "IGlobalConfigurationRouter::TryGetData found {EntryName} but it does not have field {FieldName}, skipping for router {RouterName}: {EntryValue}", EntryName, FieldName, Router->RouterName, TryPrintString(Result));
						continue;
					}
				}
				else if (GetAllowFlattenJsonObject())
				{
					UE_LOGFMT(LogGlobalConfigurationData, Verbose, "IGlobalConfigurationRouter::TryGetData found {EntryName} but it is not an object that can satisfy field request {FieldName}, assuming flattened data and returning anyway {RouterName}: {EntryValue}", EntryName, FieldName, Router->RouterName, TryPrintString(Result));
					return Result;
				}
				else
				{
					UE_LOGFMT(LogGlobalConfigurationData, Warning, "IGlobalConfigurationRouter::TryGetData found {EntryName} but it is not an object that can satisfy field request {FieldName}, skipping for router {RouterName}: {EntryValue}", EntryName, FieldName, Router->RouterName, TryPrintString(Result));
					continue;
				}
			}

			UE_LOGFMT(LogGlobalConfigurationData, VeryVerbose, "IGlobalConfigurationRouter::TryGetData found value for {EntryName} in router {RouterName}: {EntryValue}", FullEntryName, Router->RouterName, TryPrintString(Result));
			return Result;
		}
	}

	UE_LOGFMT(LogGlobalConfigurationData, VeryVerbose, "IGlobalConfigurationRouter::TryGetData found no results for {EntryName}", FullEntryName);
	return {};
}

void IGlobalConfigurationRouter::GetAllRegisteredData(TMap<FString, TMap<FString, TSharedRef<FJsonValue>>>& DataOut)
{
	check(IsInGameThread());

	TMap<FString, TSharedRef<FJsonValue>> Entries;
	for (const IGlobalConfigurationRouter* Router : IGlobalConfigurationRouter_Private::RegisteredRouters)
	{
		Entries.Empty();
		Router->GetAllDataFromRouter(Entries);

		for (TPair<FString, TSharedRef<FJsonValue>> Pair : Entries)
		{
			DataOut.FindOrAdd(Pair.Key).Add(Router->RouterName, Pair.Value);
		}
	}
}

TSharedPtr<FJsonValue> IGlobalConfigurationRouter::TryParseString(const FStringView& String)
{
	if (String.IsEmpty())
	{
		return {};
	}

	TSharedPtr<FJsonValue> JsonValue;
	IGlobalConfigurationRouter_Private::FStackJsonReader Reader(String);
	if (FJsonSerializer::Deserialize(Reader, JsonValue))
	{
		return JsonValue;
	}
	else
	{
		if (IGlobalConfigurationRouter_Private::bShowParseErrorsAsWarning)
		{
			UE_LOGFMT(LogGlobalConfigurationData, Warning, "IGlobalConfigurationRouter::TryParseString parsing {JsonString} failed with error {JsonError}, storing as raw string value instead.", String, Reader.GetErrorMessage());
		}
		else
		{
			UE_LOGFMT(LogGlobalConfigurationData, Verbose, "IGlobalConfigurationRouter::TryParseString parsing {JsonString} failed with error {JsonError}, storing as raw string value instead.", String, Reader.GetErrorMessage());
		}
	}

	// If regular parse failed then just assume it's an unquoted string
	return MakeShared<FJsonValueString>(FString(String));
}

FString IGlobalConfigurationRouter::TryPrintString(TSharedPtr<FJsonValue> Value)
{
	if (!Value.IsValid())
	{
		return {};
	}

	FString OutputString;
	IGlobalConfigurationRouter_Private::FStackJsonWriter Writer(&OutputString);
	FJsonSerializer::Serialize(Value, FString(), Writer);
	return OutputString;
}

bool IGlobalConfigurationRouter::GetAllowFlattenJsonObject()
{
	return IGlobalConfigurationRouter_Private::bAllowFlattenJsonObject;
}
