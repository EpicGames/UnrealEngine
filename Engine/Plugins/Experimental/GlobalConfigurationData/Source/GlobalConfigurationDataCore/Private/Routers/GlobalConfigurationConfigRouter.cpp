// Copyright Epic Games, Inc. All Rights Reserved.

#include "Routers/GlobalConfigurationConfigRouter.h"

#include "GlobalConfigurationDataInternal.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"

FGlobalConfigurationConfigRouter::FGlobalConfigurationConfigRouter(FString&& InConfigSection, FString&& InRouterName, int32 InPriority)
: IGlobalConfigurationRouter(MoveTemp(InRouterName), InPriority)
, ConfigSection(MoveTemp(InConfigSection))
{
	OnConfigSectionsChanged({}, {});
	FCoreDelegates::TSOnConfigSectionsChanged().AddRaw(this, &FGlobalConfigurationConfigRouter::OnConfigSectionsChanged);
}

FGlobalConfigurationConfigRouter::~FGlobalConfigurationConfigRouter()
{
	FCoreDelegates::TSOnConfigSectionsChanged().RemoveAll(this);
}

TSharedPtr<FJsonValue> FGlobalConfigurationConfigRouter::TryGetDataFromRouter(const FString& EntryName) const
{
	if (const TSharedRef<FJsonValue>* Value = StoredData.Find(EntryName))
	{
		return *Value;
	}
	return {};
}

void FGlobalConfigurationConfigRouter::GetAllDataFromRouter(TMap<FString, TSharedRef<FJsonValue>>& DataOut) const
{
	DataOut = StoredData;
}

void FGlobalConfigurationConfigRouter::OnConfigSectionsChanged(const FString& IniFilename, const TSet<FString>& SectionNames)
{
	if (SectionNames.IsEmpty() || (IniFilename == GEngineIni && SectionNames.Contains(ConfigSection)))
	{
		StoredData.Empty();

		FKeyValueSink Visitor = FKeyValueSink::CreateLambda([this](const TCHAR* Key, const TCHAR* ValueString)
			{
				if (TSharedPtr<FJsonValue> Value = TryParseString(ValueString))
				{
					StoredData.Add(Key, Value.ToSharedRef());
				}
			});

		GConfig->ForEachEntry(Visitor, *ConfigSection, GEngineIni);
	}
}