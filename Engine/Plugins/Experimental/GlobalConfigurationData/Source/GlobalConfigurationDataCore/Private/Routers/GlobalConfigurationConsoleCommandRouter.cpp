// Copyright Epic Games, Inc. All Rights Reserved.

#include "Routers/GlobalConfigurationConsoleCommandRouter.h"

#include "Dom/JsonValue.h"
#include "GlobalConfigurationDataInternal.h"
#include "HAL/IConsoleManager.h"

namespace GlobalConfigurationConfigRouter_Private
{
	TMap<FString, TSharedRef<FJsonValue>> StoredData;

	void RegisterValue(const TArray<FString>& Args)
	{
		if (Args.Num() == 2)
		{
			if (TSharedPtr<FJsonValue> Value = FGlobalConfigurationConsoleCommandRouter::TryParseString(Args[1]))
			{
				StoredData.Emplace(Args[0], Value.ToSharedRef());
				UE_LOGFMT(LogGlobalConfigurationData, Verbose, "Registering Console Command value for {EntryName}: {EntryValue}", Args[0], Args[1]);
			}
			else
			{
				UE_LOGFMT(LogGlobalConfigurationData, Error, "GCD.RegisterValue failed to parse value for {EntryName}: {EntryValue}", Args[0], Args[1]);
			}
		}
		else
		{
			UE_LOGFMT(LogGlobalConfigurationData, Verbose, "GCD.RegisterValue called with an incorrect number of arguments, did you forget quotes on the value?");
		}
	}

	void UnregisterValue(const TArray<FString>& Args)
	{
		if (!Args.IsEmpty())
		{
			StoredData.Remove(Args[0]);
		}
	}

	FAutoConsoleCommand RegisterValueCommand(
		TEXT("GCD.RegisterValue"),
		TEXT("Register a new value into the Global Configuration Data system that will override all others"),
		FConsoleCommandWithArgsDelegate::CreateStatic(&RegisterValue));

	FAutoConsoleCommand UnregisterValueCommand(
		TEXT("GCD.UnregisterValue"),
		TEXT("Remove a value from the Global Configuration Data system that was overriding all others"),
		FConsoleCommandWithArgsDelegate::CreateStatic(&UnregisterValue));
}

FGlobalConfigurationConsoleCommandRouter::FGlobalConfigurationConsoleCommandRouter()
: IGlobalConfigurationRouter(TEXT("ConsoleCommand"), INT32_MAX)
{
}

TSharedPtr<FJsonValue> FGlobalConfigurationConsoleCommandRouter::TryGetDataFromRouter(const FString& EntryName) const
{
	if (const TSharedRef<FJsonValue>* Value = GlobalConfigurationConfigRouter_Private::StoredData.Find(EntryName))
	{
		return *Value;
	}
	return {};
}

void FGlobalConfigurationConsoleCommandRouter::GetAllDataFromRouter(TMap<FString, TSharedRef<FJsonValue>>& DataOut) const
{
	DataOut = GlobalConfigurationConfigRouter_Private::StoredData;
}
