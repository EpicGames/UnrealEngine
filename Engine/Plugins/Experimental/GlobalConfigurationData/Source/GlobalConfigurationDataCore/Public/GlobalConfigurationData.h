// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"

class UStruct;
class UObject;

/**
 * GlobalConfigurationData is a system that allows querying for data from a unique name.
 * Multiple data routers can be registered so a user doesn't have to know where it's coming from.
 * This system allows use to use a multitude of systems to deliver configuration to the user.
 * 
 * There are two built-in routers that use config files or console commands. Other routers exist
 * in other plugins, or users can create their own, to pull data from other sources for use as hotfixes
 * data experiments, etc.
 */
namespace UE::GlobalConfigurationData
{
	GLOBALCONFIGURATIONDATACORE_API bool TryGetDataOfType(const FString& EntryName, const UStruct* Type, void* DataOut);

	GLOBALCONFIGURATIONDATACORE_API bool TryGetData(const FString& EntryName, bool& bValueOut);
	GLOBALCONFIGURATIONDATACORE_API bool TryGetData(const FString& EntryName, int32& ValueOut);
	GLOBALCONFIGURATIONDATACORE_API bool TryGetData(const FString& EntryName, float& ValueOut);
	GLOBALCONFIGURATIONDATACORE_API bool TryGetData(const FString& EntryName, FString& ValueOut);
	GLOBALCONFIGURATIONDATACORE_API bool TryGetData(const FString& EntryName, FText& ValueOut);

	template<typename Type UE_REQUIRES(TModels_V<CStaticStructProvider, Type>)>
	bool TryGetData(const FString& EntryName, Type& DataOut)
	{
		return TryGetDataOfType(EntryName, Type::StaticStruct(), &DataOut);
	}

	template<typename Type UE_REQUIRES(std::is_base_of_v<UObject, Type>)>
	bool TryGetData(const FString& EntryName, Type* DataOut)
	{
		return TryGetDataOfType(EntryName, Type::StaticClass(), DataOut);
	}

	template <typename Type>
	Type GetDataWithDefault(const FString& EntryName, const Type& DefaultValue)
	{
		Type Data;
		if (TryGetData(EntryName, Data))
		{
			return Data;
		}

		return DefaultValue;
	}
}