// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"

class FJsonValue;

/** The Router is a class that is used to lookup data from various systems in an order determined by their priority
 * Data should be stored as string values.
 * 	For primitive types it will use console variable parsing rules.
 * 	For complex types the data should be stored in a Json format.
 * 
 * Routers auto register themselves, so just make sure they exist somewhere and that's it.
 */
class IGlobalConfigurationRouter
{
public:
	GLOBALCONFIGURATIONDATACORE_API IGlobalConfigurationRouter(FString&& InRouterName, int32 InPriority);
	GLOBALCONFIGURATIONDATACORE_API virtual ~IGlobalConfigurationRouter();

	GLOBALCONFIGURATIONDATACORE_API static TSharedPtr<FJsonValue> TryGetData(const FString& EntryName);
	
	// Map of Entry Name to a map of Value by Router Name
	GLOBALCONFIGURATIONDATACORE_API static void GetAllRegisteredData(TMap<FString, TMap<FString, TSharedRef<FJsonValue>>>& DataOut);

	GLOBALCONFIGURATIONDATACORE_API static TSharedPtr<FJsonValue> TryParseString(const FStringView& String);
	GLOBALCONFIGURATIONDATACORE_API static FString TryPrintString(TSharedPtr<FJsonValue> Value);

protected:
	// If enabled allow routers to flatten json objects to a single value. The core system will allow field name references to accept single value returns.
	GLOBALCONFIGURATIONDATACORE_API static bool GetAllowFlattenJsonObject();
	
	virtual TSharedPtr<FJsonValue> TryGetDataFromRouter(const FString& EntryName) const = 0;
	virtual void GetAllDataFromRouter(TMap<FString, TSharedRef<FJsonValue>>& DataOut) const = 0;

private:
	FString RouterName;

	/** Arbitrary value used for sorting.
	 * The built-in routers are at the extreme ends of this range,
	 * Console Command router being the highest priority (INT32_MAX)
	 * Config Router being the lowest priority (INT32_MIN)
	 * 
	 * The idea is that a config value is baseline and baked into the product where the console command is used for testing or hotfixing values at the very highest.
	 * User routers should fall in between these two extremes and manage their own priority levels to get the desired outcome.
	 */ 
	int32 Priority;
};
