// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GlobalConfigurationRouter.h"

/**
 * Data Router that pulls from the section GlobalConfigurationData in GEngineIni
 */
class FGlobalConfigurationConfigRouter : public IGlobalConfigurationRouter
{
public:
	FGlobalConfigurationConfigRouter(FString&& InConfigSection, FString&& InRouterName, int32 InPriority);
	~FGlobalConfigurationConfigRouter();

	virtual TSharedPtr<FJsonValue> TryGetDataFromRouter(const FString& EntryName) const override;
	virtual void GetAllDataFromRouter(TMap<FString, TSharedRef<FJsonValue>>& DataOut) const override;
	
private:
	void OnConfigSectionsChanged(const FString& IniFilename, const TSet<FString>& SectionNames);

	FString ConfigSection;
	TMap<FString, TSharedRef<FJsonValue>> StoredData;
};
