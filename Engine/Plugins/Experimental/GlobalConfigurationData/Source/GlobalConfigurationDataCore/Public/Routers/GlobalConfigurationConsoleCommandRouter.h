// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GlobalConfigurationRouter.h"

/**
 * Data router that uses console commands to register data with the system:
 * GCD.RegisterValue Name "Value"
 * GCD.UnregisterValue Name
 */
class FGlobalConfigurationConsoleCommandRouter : public IGlobalConfigurationRouter
{
public:
	FGlobalConfigurationConsoleCommandRouter();

	virtual TSharedPtr<FJsonValue> TryGetDataFromRouter(const FString& EntryName) const override;
	virtual void GetAllDataFromRouter(TMap<FString, TSharedRef<FJsonValue>>& DataOut) const override;
};
